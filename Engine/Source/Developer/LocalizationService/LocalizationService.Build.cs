// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LocalizationService : ModuleRules
{
	public LocalizationService(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
                "Engine",
				"InputCore",
			}
		);

        if (!(Target.IsInPlatformGroup(UnrealPlatformGroup.Linux) && Target.Type == TargetType.Program))
        {
            PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "Slate",
                    "SlateCore",
                }
            );
        }

        if (Target.bBuildEditor)
        {
			PrivateDependencyModuleNames.AddRange(
                new string[] {
					"EditorFramework",
					"UnrealEd",
                    "PropertyEditor",
            }
            );
        }

		if (Target.bBuildDeveloperTools)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"MessageLog",
				}
			);
		}
	}
}
