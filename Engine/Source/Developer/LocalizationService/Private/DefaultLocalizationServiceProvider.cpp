// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultLocalizationServiceProvider.h"
#include "UObject/WeakObjectPtr.h"
#include "Logging/MessageLog.h"

#define LOCTEXT_NAMESPACE "DefaultLocalizationServiceProvider"

void FDefaultLocalizationServiceProvider::Init(bool bForceConnection)
{
	FMessageLog("LocalizationService").Info(LOCTEXT("LocalizationServiceDisabled", "Localization service is disabled"));
}

void FDefaultLocalizationServiceProvider::Close()
{

}

FText FDefaultLocalizationServiceProvider::GetStatusText() const
{
	return LOCTEXT("LocalizationServiceDisabled", "Localization service is disabled");
}

bool FDefaultLocalizationServiceProvider::IsAvailable() const
{
	return false;
}

bool FDefaultLocalizationServiceProvider::IsEnabled() const
{
	return false;
}

const FName& FDefaultLocalizationServiceProvider::GetName(void) const
{
	static FName ProviderName("None"); 
	return ProviderName; 
}

const FText FDefaultLocalizationServiceProvider::GetDisplayName() const
{
	return LOCTEXT("DefaultLocalizationServiceProviderDisplayName", "None");
}

ELocalizationServiceOperationCommandResult::Type FDefaultLocalizationServiceProvider::GetState(const TArray<FLocalizationServiceTranslationIdentifier>& InTranslationIds, TArray< TSharedRef<ILocalizationServiceState, ESPMode::ThreadSafe> >& OutState, ELocalizationServiceCacheUsage::Type InStateCacheUsage)
{
	return ELocalizationServiceOperationCommandResult::Failed;
}

ELocalizationServiceOperationCommandResult::Type FDefaultLocalizationServiceProvider::Execute(const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation, const TArray<FLocalizationServiceTranslationIdentifier>& InTranslationIds, ELocalizationServiceOperationConcurrency::Type InConcurrency, const FLocalizationServiceOperationComplete& InOperationCompleteDelegate)
{
	return ELocalizationServiceOperationCommandResult::Failed;
}

bool FDefaultLocalizationServiceProvider::CanCancelOperation( const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation ) const
{
	return false;
}

void FDefaultLocalizationServiceProvider::CancelOperation( const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation )
{
}

void FDefaultLocalizationServiceProvider::Tick()
{

}

#if LOCALIZATION_SERVICES_WITH_SLATE
void FDefaultLocalizationServiceProvider::CustomizeSettingsDetails(IDetailCategoryBuilder& DetailCategoryBuilder) const
{

}

void FDefaultLocalizationServiceProvider::CustomizeTargetDetails(IDetailCategoryBuilder& DetailCategoryBuilder, TWeakObjectPtr<ULocalizationTarget> LocalizationTarget) const
{

}

void FDefaultLocalizationServiceProvider::CustomizeTargetToolbar(TSharedRef<FExtender>& MenuExtender, TWeakObjectPtr<ULocalizationTarget> LocalizationTarget) const
{

}

void FDefaultLocalizationServiceProvider::CustomizeTargetSetToolbar(TSharedRef<FExtender>& MenuExtender, TWeakObjectPtr<ULocalizationTargetSet> LocalizationTargetSet) const
{

}

#endif //LOCALIZATION_SERVICES_WITH_SLATE

#undef LOCTEXT_NAMESPACE
