// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "ILocalizationServiceOperation.h"
#include "ILocalizationServiceState.h"
#include "ILocalizationServiceProvider.h"

class IDetailCategoryBuilder;
class ULocalizationTarget;
class ULocalizationTargetSet;

/**
 * Default localization service provider implementation - "None"
 */
class FDefaultLocalizationServiceProvider : public ILocalizationServiceProvider
{
public:
	// ILocalizationServiceProvider implementation
	virtual void Init(bool bForceConnection = true) override;
	virtual void Close() override;
	virtual FText GetStatusText() const override;
	virtual bool IsAvailable() const override;
	virtual bool IsEnabled() const override;
	virtual const FName& GetName(void) const override;
	virtual const FText GetDisplayName() const override;
	virtual ELocalizationServiceOperationCommandResult::Type GetState(const TArray<FLocalizationServiceTranslationIdentifier>& InTranslationIds, TArray< TSharedRef<ILocalizationServiceState, ESPMode::ThreadSafe> >& OutState, ELocalizationServiceCacheUsage::Type InStateCacheUsage) override;
	virtual ELocalizationServiceOperationCommandResult::Type Execute(const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation, const TArray<FLocalizationServiceTranslationIdentifier>& InTranslationIds, ELocalizationServiceOperationConcurrency::Type InConcurrency = ELocalizationServiceOperationConcurrency::Synchronous, const FLocalizationServiceOperationComplete& InOperationCompleteDelegate = FLocalizationServiceOperationComplete()) override;
	virtual bool CanCancelOperation( const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation ) const override;
	virtual void CancelOperation( const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation ) override;
	virtual void Tick() override;
#if LOCALIZATION_SERVICES_WITH_SLATE
	virtual void CustomizeSettingsDetails(IDetailCategoryBuilder& DetailCategoryBuilder) const override;
	virtual void CustomizeTargetDetails(IDetailCategoryBuilder& DetailCategoryBuilder, TWeakObjectPtr<ULocalizationTarget> LocalizationTarget) const override;
	virtual void CustomizeTargetToolbar(TSharedRef<FExtender>& MenuExtender, TWeakObjectPtr<ULocalizationTarget> LocalizationTarget) const override;
	virtual void CustomizeTargetSetToolbar(TSharedRef<FExtender>& MenuExtender, TWeakObjectPtr<ULocalizationTargetSet> LocalizationTargetSet) const override;
#endif // LOCALIZATION_SERVICES_WITH_SLATE
};

