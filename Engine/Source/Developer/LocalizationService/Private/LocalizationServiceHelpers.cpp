// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationServiceHelpers.h"
#include "ILocalizationServiceProvider.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "ILocalizationServiceModule.h"

#define LOCTEXT_NAMESPACE "LocalizationServiceHelpers"

namespace LocalizationServiceHelpers
{

	const FString& GetSettingsIni()
	{
		if (ILocalizationServiceModule::Get().GetUseGlobalSettings())
		{
			return GetGlobalSettingsIni();
		}
		else
		{
			static FString LocalizationServiceSettingsIni;
			if (LocalizationServiceSettingsIni.Len() == 0)
			{
				FConfigContext Context = FConfigContext::ReadIntoGConfig();
				Context.Load(TEXT("LocalizationServiceSettings"), LocalizationServiceSettingsIni);
			}
			return LocalizationServiceSettingsIni;
		}
	}

	const FString& GetGlobalSettingsIni()
	{
		static FString LocalizationServiceGlobalSettingsIni;
		if (LocalizationServiceGlobalSettingsIni.Len() == 0)
		{
			FConfigContext Context = FConfigContext::ReadIntoGConfig();
			Context.GeneratedConfigDir = FPaths::EngineSavedDir() + TEXT("Config/");
			Context.Load(TEXT("LocalizationServiceSettings"), LocalizationServiceGlobalSettingsIni);
		}
		return LocalizationServiceGlobalSettingsIni;
	}


	//LOCALIZATIONSERVICE_API extern bool CommitTranslation(const FString& Culture, const FString& Namespace, const FString& Source, const FString& Translation)
	//{
	//	if (!Culture.IsValid() || Culture->GetName().IsEmpty())
	//	{
	//		FMessageLog("LocalizationService").Error(LOCTEXT("CultureNotSpecified", "Culture not specified"));
	//		return false;
	//	}

	//	if (!ILocalizationServiceModule::Get().IsEnabled())
	//	{
	//		FMessageLog("LocalizationService").Error(LOCTEXT("LocalizationServiceDisabled", "Localization Service is not enabled."));
	//		return false;
	//	}

	//	if (!ILocalizationServiceModule::Get().GetProvider().IsAvailable())
	//	{
	//		FMessageLog("SourceControl").Error(LOCTEXT("ILocalizationServiceUnavailable", "Localization Ser is currently not available."));
	//		return false;
	//	}

	//	bool bSuccessfullyCheckedOut = false;

	//	ILocalizationServiceProvider& Provider = ILocalizationServiceModule::Get().GetProvider();
	//	//TArray<TSharedRef<ILocalizationServiceState, ESPMode::ThreadSafe>> TranslationStates;
	//	TSharedPtr<ILocalizationServiceState, ESPMode::ThreadSafe> LocalizationServiceState = Provider.GetState(FLocalizationServiceTranslationIdentifier(Culture, Namespace, Source), ELocalizationServiceCacheUsage::ForceUpdate);

	//	if (LocalizationServiceState.IsValid())
	//	{
	//		// TODO: Finish
	//	}
	//}
}

FScopedLocalizationService::FScopedLocalizationService()
{
	ILocalizationServiceModule::Get().GetProvider().Init();
}

FScopedLocalizationService::~FScopedLocalizationService()
{
	ILocalizationServiceModule::Get().GetProvider().Close();
}

ILocalizationServiceProvider& FScopedLocalizationService::GetProvider()
{
	return ILocalizationServiceModule::Get().GetProvider();
}

#undef LOCTEXT_NAMESPACE
