// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationServiceModule.h"

#if WITH_UNREAL_DEVELOPER_TOOLS
#include "MessageLogModule.h"
#endif
 
#include "Features/IModularFeatures.h"

DEFINE_LOG_CATEGORY(LogLocalizationService);

#define LOCTEXT_NAMESPACE "LocalizationService"

static const FName LocalizationServiceFeatureName("LocalizationService");

namespace LocalizationServiceConstants
{
	/** The maximum number of translation status requests we should dispatch in a tick */
	const int32 MaxStatusDispatchesPerTick = 64;
}

FLocalizationServiceModule::FLocalizationServiceModule()
	: CurrentLocalizationServiceProvider(NULL)
{
}

void FLocalizationServiceModule::StartupModule()
{
	// load our settings
	LocalizationServiceSettings.LoadSettings();

	// Register to check for translation service features
	IModularFeatures::Get().OnModularFeatureRegistered().AddRaw(this, &FLocalizationServiceModule::HandleModularFeatureRegistered);
	IModularFeatures::Get().OnModularFeatureUnregistered().AddRaw(this, &FLocalizationServiceModule::HandleModularFeatureUnregistered);

	// bind default provider to editor
	IModularFeatures::Get().RegisterModularFeature( LocalizationServiceFeatureName, &DefaultLocalizationServiceProvider );

#if WITH_UNREAL_DEVELOPER_TOOLS
	// create a message log for translation service to use
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.RegisterLogListing("LocalizationService", LOCTEXT("LocalizationServiceLogLabel", "Localization Service"));
#endif
}

void FLocalizationServiceModule::ShutdownModule()
{
	// close the current provider
	GetProvider().Close();

#if WITH_UNREAL_DEVELOPER_TOOLS
	// unregister message log
	if(FModuleManager::Get().IsModuleLoaded("MessageLog"))
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogModule.UnregisterLogListing("LocalizationService");
	}
#endif

	// unbind default provider from editor
	IModularFeatures::Get().UnregisterModularFeature( LocalizationServiceFeatureName, &DefaultLocalizationServiceProvider );

	// we don't care about modular features any more
	IModularFeatures::Get().OnModularFeatureRegistered().RemoveAll(this);
	IModularFeatures::Get().OnModularFeatureUnregistered().RemoveAll(this);
}

void FLocalizationServiceModule::SaveSettings()
{
	LocalizationServiceSettings.SaveSettings();
}

void FLocalizationServiceModule::InitializeLocalizationServiceProviders()
{
	int32 LocalizationServiceCount = IModularFeatures::Get().GetModularFeatureImplementationCount(LocalizationServiceFeatureName);
	if( LocalizationServiceCount > 0 )
	{
		FString PreferredLocalizationServiceProvider = LocalizationServiceSettings.GetProvider();
		TArray<ILocalizationServiceProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<ILocalizationServiceProvider>(LocalizationServiceFeatureName);
		for(auto It(Providers.CreateIterator()); It; It++)
		{ 
			ILocalizationServiceProvider* Provider = *It;
			if(PreferredLocalizationServiceProvider == Provider->GetName().ToString())
			{
				CurrentLocalizationServiceProvider = Provider;
				break;
			}
		}

		// no provider found of this name, default to the first one
		if( CurrentLocalizationServiceProvider == NULL )
		{
			CurrentLocalizationServiceProvider = &DefaultLocalizationServiceProvider;
		}
	}

	check(CurrentLocalizationServiceProvider);

	CurrentLocalizationServiceProvider->Init(false);	// Don't force a connection here, as its synchronous. Let the user establish a connection.
}

void FLocalizationServiceModule::Tick()
{	
	ILocalizationServiceProvider& Provider = GetProvider();

	// tick the provider, so any operation results can be read back
	Provider.Tick();
	
	// don't allow background status updates when disabled 
	if(Provider.IsEnabled())
	{
		//// check for any pending dispatches
		//if(PendingStatusUpdateTranslations.Num() > 0)
		//{
		//	// grab a batch of translations
		//	TArray<FLocalizationServiceTranslationIdentifier> TranslationsToDispatch;
		//	for(auto Iter(PendingStatusUpdateTranslations.CreateConstIterator()); Iter; Iter++)
		//	{
		//		if(FilesToDispatch.Num() >= LocalizationServiceConstants::MaxStatusDispatchesPerTick)
		//		{
		//			break;
		//		}
		//		FilesToDispatch.Add(*Iter);
		//	}

		//	if(FilesToDispatch.Num() > 0)
		//	{
		//		// remove the files we are dispatching so we don't try again
		//		PendingStatusUpdateTranslations.RemoveAt(0, TranslationsToDispatch.Num());

		//		// dispatch update
		//		Provider.Execute(ILocalizationServiceOperation::Create<FUpdateStatus>(), TranslationsToDispatch, EConcurrency::Asynchronous);
		//	}
		//}
	}
}


//void FLocalizationServiceModule::QueueStatusUpdate(const TArray<FLocalizationServiceTranslationIdentifier>& InTranslationIds)
//{
//	if(IsEnabled())
//	{
//		for(auto It(InTranslationIds.CreateConstIterator()); It; It++)
//		{
//			QueueStatusUpdate(*It);
//		}
//	}
//}

bool FLocalizationServiceModule::IsEnabled() const
{
	return GetProvider().IsEnabled();
}

ILocalizationServiceProvider& FLocalizationServiceModule::GetProvider() const
{
	return *CurrentLocalizationServiceProvider;
}

void FLocalizationServiceModule::SetProvider( const FName& InName )
{
	TArray<ILocalizationServiceProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<ILocalizationServiceProvider>(LocalizationServiceFeatureName);
	for(auto It(Providers.CreateIterator()); It; It++)
	{
		ILocalizationServiceProvider* Provider = *It;
		if(InName == Provider->GetName())
		{
			SetCurrentLocalizationServiceProvider(*Provider);
			return;
		}
	}

	UE_LOGF(LogLocalizationService, Fatal, "Tried to set unknown translation service provider: %ls", *InName.ToString());
}

void FLocalizationServiceModule::ClearCurrentLocalizationServiceProvider()
{
	if( CurrentLocalizationServiceProvider != NULL )
	{
		CurrentLocalizationServiceProvider->Close();
		CurrentLocalizationServiceProvider = &DefaultLocalizationServiceProvider;
	}
}

int32 FLocalizationServiceModule::GetNumLocalizationServiceProviders()
{
	return IModularFeatures::Get().GetModularFeatureImplementationCount(LocalizationServiceFeatureName);
}

void FLocalizationServiceModule::SetCurrentLocalizationServiceProvider(int32 ProviderIndex)
{
	TArray<ILocalizationServiceProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<ILocalizationServiceProvider>(LocalizationServiceFeatureName);
	check(Providers.IsValidIndex(ProviderIndex));
	SetCurrentLocalizationServiceProvider(*Providers[ProviderIndex]);
}

void FLocalizationServiceModule::SetCurrentLocalizationServiceProvider(ILocalizationServiceProvider& InProvider)
{
	// see if we are switching or not
	if(&InProvider == CurrentLocalizationServiceProvider)
	{
		return;
	}

	ClearCurrentLocalizationServiceProvider();

	CurrentLocalizationServiceProvider = &InProvider;
	CurrentLocalizationServiceProvider->Init(false);	// Don't force a connection here, as its synchronous. Let the user establish a connection.

	LocalizationServiceSettings.SetProvider(CurrentLocalizationServiceProvider->GetName().ToString());

	SaveSettings();
}

FName FLocalizationServiceModule::GetLocalizationServiceProviderName(int32 ProviderIndex)
{
	TArray<ILocalizationServiceProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<ILocalizationServiceProvider>(LocalizationServiceFeatureName);
	check(Providers.IsValidIndex(ProviderIndex));
	return Providers[ProviderIndex]->GetName();
}

void FLocalizationServiceModule::HandleModularFeatureRegistered(const FName& Type, IModularFeature* ModularFeature)
{
	if(Type == LocalizationServiceFeatureName)
	{
		InitializeLocalizationServiceProviders();
	}
}

void FLocalizationServiceModule::HandleModularFeatureUnregistered(const FName& Type, IModularFeature* ModularFeature)
{
	if(Type == LocalizationServiceFeatureName && CurrentLocalizationServiceProvider == static_cast<ILocalizationServiceProvider*>(ModularFeature))
	{
		ClearCurrentLocalizationServiceProvider();
	}
}

bool FLocalizationServiceModule::GetUseGlobalSettings() const
{
	return LocalizationServiceSettings.GetUseGlobalSettings();
}

void FLocalizationServiceModule::SetUseGlobalSettings(bool bIsUseGlobalSettings)
{
	LocalizationServiceSettings.SetUseGlobalSettings(bIsUseGlobalSettings);
}	

IMPLEMENT_MODULE( FLocalizationServiceModule, LocalizationService );

#undef LOCTEXT_NAMESPACE
