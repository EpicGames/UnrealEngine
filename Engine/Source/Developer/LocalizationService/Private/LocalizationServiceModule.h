// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Interface for talking to source control clients
 */

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "ILocalizationServiceModule.h"
#include "LocalizationServiceSettings.h"
#include "DefaultLocalizationServiceProvider.h"

class FLocalizationServiceModule : public ILocalizationServiceModule
{
public:
	FLocalizationServiceModule();

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** ILocalizationServiceModule implementation */
	virtual void Tick() override;
	virtual bool IsEnabled() const override;
	virtual ILocalizationServiceProvider& GetProvider() const override;
	virtual void SetProvider( const FName& InName ) override;
	virtual bool GetUseGlobalSettings() const override;
	virtual void SetUseGlobalSettings(bool bIsUseGlobalSettings) override;

	/** Save the settings to the ini file */
	void SaveSettings();

	/**
	 * Get the number of currently registered translation service providers.
	 */
	int32 GetNumLocalizationServiceProviders();

	/**
	 * Set the current translation service provider by index.
	 */
	void SetCurrentLocalizationServiceProvider(int32 ProviderIndex);

	/**
	 * Get the name of the translation service provider at the specified index.
	 */
	FName GetLocalizationServiceProviderName(int32 ProviderIndex);

	/**
	 * Gets a reference to the translation service module instance.
	 *
	 * @return A reference to the translation service module.
	 */
	static FLocalizationServiceModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FLocalizationServiceModule>("LocalizationService");
	}

private:
	/** Refresh & initialize the current translation service provider */
	void InitializeLocalizationServiceProviders();

	/** Close the current translation service provider & set the current to default - 'None' */
	void ClearCurrentLocalizationServiceProvider();

	/** Set the current translation service provider to the passed-in value */
	void SetCurrentLocalizationServiceProvider(ILocalizationServiceProvider& InProvider);

	/** Delegate handling when translation service features are registered */
	void HandleModularFeatureRegistered(const FName& Type, IModularFeature* ModularFeature);

	/** Delegate handling when translation service features are unregistered */
	void HandleModularFeatureUnregistered(const FName& Type, IModularFeature* ModularFeature);

private:
	/** The settings object */
	FLocalizationServiceSettings LocalizationServiceSettings;

	/** Current translation service provider */
	ILocalizationServiceProvider* CurrentLocalizationServiceProvider;

	/** Source control provider we use if there are none registered */
	FDefaultLocalizationServiceProvider DefaultLocalizationServiceProvider;

	///** Translations pending a status update */
	//TArray < FLocalizationServiceTranslationIdentifier > PendingStatusUpdateTranslations;
};
