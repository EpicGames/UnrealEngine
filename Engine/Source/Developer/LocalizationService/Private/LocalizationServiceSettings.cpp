// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationServiceSettings.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "ILocalizationServiceProvider.h"
#include "LocalizationServiceHelpers.h"

namespace LocalizationServiceSettingsConstants
{

/** The section of the ini file we load our settings from */
static const FString SettingsSection = TEXT("LocalizationService.LocalizationServiceSettings");

}


const FString& FLocalizationServiceSettings::GetProvider() const
{
	return Provider;
}

void FLocalizationServiceSettings::SetProvider(const FString& InString)
{
	Provider = InString;
}

bool FLocalizationServiceSettings::GetUseGlobalSettings() const
{
	return bUseGlobalSettings;
}

void FLocalizationServiceSettings::SetUseGlobalSettings(bool bInUseGlobalSettings)
{
	bUseGlobalSettings = bInUseGlobalSettings;
}

void FLocalizationServiceSettings::LoadSettings()
{
	// make sure we load the global ini first
	const FString& GlobalIniFile = LocalizationServiceHelpers::GetGlobalSettingsIni();
	GConfig->GetBool(*LocalizationServiceSettingsConstants::SettingsSection, TEXT("UseGlobalSettings"), bUseGlobalSettings, GlobalIniFile);

	TArray<FString> Tokens;
	TArray<FString> Switches;
	FCommandLine::Parse( FCommandLine::Get(), Tokens, Switches );
	TMap<FString, FString> SwitchPairs;
	for (int32 SwitchIdx = Switches.Num() - 1; SwitchIdx >= 0; --SwitchIdx)
	{
		FString& Switch = Switches[SwitchIdx];
		TArray<FString> SplitSwitch;
		if (2 == Switch.ParseIntoArray(SplitSwitch, TEXT("="), true))
		{
			SwitchPairs.Add(SplitSwitch[0], SplitSwitch[1].TrimQuotes());
			Switches.RemoveAt(SwitchIdx);
		}
	}

	if( SwitchPairs.Contains( TEXT("LocalizationServiceProvider") ) )
	{
		Provider = SwitchPairs[TEXT("LocalizationServiceProvider")];
	}
	else
	{
		const FString& IniFile = LocalizationServiceHelpers::GetSettingsIni();
		GConfig->GetString(*LocalizationServiceSettingsConstants::SettingsSection, TEXT("Provider"), Provider, IniFile);
	}
}

void FLocalizationServiceSettings::SaveSettings() const
{
	const FString& IniFile = LocalizationServiceHelpers::GetSettingsIni();
	const FString& GlobalIniFile = LocalizationServiceHelpers::GetGlobalSettingsIni();
	GConfig->SetString(*LocalizationServiceSettingsConstants::SettingsSection, TEXT("Provider"), *Provider, IniFile);
	GConfig->SetBool(*LocalizationServiceSettingsConstants::SettingsSection, TEXT("UseGlobalSettings"), bUseGlobalSettings, GlobalIniFile);	
}
