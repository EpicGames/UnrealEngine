// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScopedLocalizationServiceProgress.h"
#include "Misc/App.h"
#include "ILocalizationServiceProvider.h"

#if LOCALIZATION_SERVICES_WITH_SLATE
	#include "Framework/Application/SlateApplication.h"
	#include "Widgets/Layout/SBorder.h"
	#include "Widgets/Images/SImage.h"
	#include "Widgets/Text/STextBlock.h"
	#include "Widgets/Layout/SBox.h"
	#include "Widgets/Layout/SUniformGridPanel.h"
	#include "Widgets/Input/SButton.h"
	#include "Framework/Docking/TabManager.h"
	#include "Styling/AppStyle.h"
#endif // LOCALIZATION_SERVICES_WITH_SLATE

#if LOCALIZATION_SERVICES_WITH_SLATE
#include "Widgets/Images/SThrobber.h"

#define LOCTEXT_NAMESPACE "LocalizationServiceProgress"

namespace LocalizationServiceConstants
{
	/** The time (in seconds) we wait before letting the user know that an operation is taking a while */
	float OperationTimeOut = 10.0f;
}

class SLocalizationServiceProgress : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SLocalizationServiceProgress) {}
	
	/** The text to display */
	SLATE_ATTRIBUTE(FText, Text)

	/** The delegate to call when the cancel button is clicked */
	SLATE_ARGUMENT(FSimpleDelegate, OnCancelled)

	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs)
	{
		OnCancelled = InArgs._OnCancelled;
		bCancelClicked = false;
		TimeStamp = FPlatformTime::Seconds();

		SBorder::Construct( SBorder::FArguments()
		.BorderImage( FAppStyle::GetBrush("ChildWindow.Background") )
		.Padding(16.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(500.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 8.0f)
				[
					SNew(STextBlock)
					.Text(InArgs._Text)
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 8.0f)
				[
					SNew(SHorizontalBox)
					.Visibility(this, &SLocalizationServiceProgress::GetWarningVisibility)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("LocalizationService.ProgressWindow.Warning"))
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("LongTaskMessage", "Operation is taking a long time to complete. Click cancel to stop the current operation, you can try again later."))
						.WrapTextAt(450.0f)
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 8.0f, 0.0f, 0.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					[
						SNew(SThrobber)
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.AutoWidth()
					[
						// buttons
						SNew(SUniformGridPanel)
						.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
						.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
						.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
						+SUniformGridPanel::Slot(0, 0)
						[
							SNew(SButton)
							.HAlign(HAlign_Center)
							.Text(LOCTEXT("ContinueButtonLabel", "Continue"))
							.OnClicked(this, &SLocalizationServiceProgress::OnContinueClicked)
							.Visibility(this, &SLocalizationServiceProgress::GetWarningVisibility)
						]
						+SUniformGridPanel::Slot(1, 0)
						[
							SNew(SButton)
							.HAlign(HAlign_Center)
							.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
							.OnClicked(this, &SLocalizationServiceProgress::OnCancelClicked)
							.IsEnabled(this, &SLocalizationServiceProgress::IsCancelEnabled)
							.Visibility(this, &SLocalizationServiceProgress::GetCancelVisibility)
						]
					]
				]
			]
		]);
	}

private:

	/** Delegate used to get the visibility of the cancel button */
	EVisibility GetCancelVisibility() const
	{
		return OnCancelled.IsBound() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	/** Delegate used to get the visibility of the warning text */
	EVisibility GetWarningVisibility() const
	{
		if(!OnCancelled.IsBound())
		{
			return EVisibility::Collapsed;
		}
		else if(FPlatformTime::Seconds() - TimeStamp > LocalizationServiceConstants::OperationTimeOut)
		{
			return EVisibility::Visible;
		}

		return EVisibility::Hidden;
	}

	/** Handler for the continue button */
	FReply OnContinueClicked()
	{
		TimeStamp = FPlatformTime::Seconds();
		return FReply::Handled();
	}

	/** Handler for the cancel button */
	FReply OnCancelClicked()
	{
		bCancelClicked = true;
		OnCancelled.ExecuteIfBound();
		return FReply::Handled();
	}

	/** Delegate used to get the enabled state of the cancel button */
	bool IsCancelEnabled() const
	{
		return !bCancelClicked;
	}

private:

	/** Flag used to disable the cancel button once clicked */
	bool bCancelClicked;

	/** The timer we use to determine when to display the 'long task' message */
	float TimeStamp;

	/** The delegate to call when the cancel button is clicked */
	FSimpleDelegate OnCancelled;
};

FScopedLocalizationServiceProgress::FScopedLocalizationServiceProgress(const FText& InText, const FSimpleDelegate& InOnCancelled)
{
	if(!(FApp::IsUnattended() || IsRunningCommandlet()) && !InText.IsEmpty())
	{
		TSharedRef<SWindow> Window = SNew(SWindow)
			.IsPopupWindow(true)
			.SupportsMaximize(false) 
			.SupportsMinimize(false)
			.CreateTitleBar(false)
			.SizingRule(ESizingRule::Autosized);

		WindowPtr = Window;

		Window->SetContent(
			SNew(SLocalizationServiceProgress)
			.Text(InText)
			.OnCancelled(InOnCancelled) 
		);
			
		TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
		FSlateApplication::Get().AddModalWindow(Window, RootWindow, true);

		Window->ShowWindow();	
	
		Tick();
	}
}

FScopedLocalizationServiceProgress::~FScopedLocalizationServiceProgress()
{
	if(WindowPtr.IsValid())
	{
		WindowPtr.Pin()->RequestDestroyWindow();
	}
}

void FScopedLocalizationServiceProgress::Tick()
{
	if (!(FApp::IsUnattended() || IsRunningCommandlet()) && WindowPtr.IsValid() && FSlateApplication::Get().CanDisplayWindows())
	{
		// Tick Slate application
		FSlateApplication::Get().Tick();

		// Sync the game thread and the render thread
		FSlateApplication::Get().GetRenderer()->Sync();
	}
}
#else
FScopedLocalizationServiceProgress::FScopedLocalizationServiceProgress(const FText& InText, const FSimpleDelegate& InOnCancelled)
{

}

FScopedLocalizationServiceProgress::~FScopedLocalizationServiceProgress()
{

}

void FScopedLocalizationServiceProgress::Tick()
{

}
#endif // LOCALIZATION_SERVICES_WITH_SLATE

#undef LOCTEXT_NAMESPACE
