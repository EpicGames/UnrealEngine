// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class ILocalizationServiceProvider;

LOCALIZATIONSERVICE_API DECLARE_LOG_CATEGORY_EXTERN(LogLocalizationService, Log, All);

/** Delegate called when the localization service login window is closed. Parameter determines if service is enabled or not */
DECLARE_DELEGATE_OneParam( FLocalizationServiceLoginClosed, bool );

/**
 * The modality of the login window.
 */
namespace ELocalizationServiceLoginWindowMode
{
	enum Type
	{
		Modal,
		Modeless
	};
};


/**
 * Login window startup behavior
 */
namespace ELocalizationServiceOnLoginWindowStartup
{
	enum Type
	{
		ResetProviderToNone,
		PreserveProvider
	};
};


/**
 * Interface for talking to localization service providers
 */
class ILocalizationServiceModule : public IModuleInterface
{
public:

	/**
	 * Tick the localization service module.
	 * This is responsible for dispatching batched/queued status requests & for calling ILocalizationServiceProvider::Tick()
	 */
	virtual void Tick() = 0;

	///**
	// * Queues translations to have their status updated in the background.
	// * @param	InTranslationIds	The transaltions to queue.
	// */
	//virtual void QueueStatusUpdate(const TArray<FLocalizationServiceTranslationIdentifier>& InTranslationIds) = 0;

	/**
	 * Check whether localization service is enabled.
	 */
	virtual bool IsEnabled() const = 0;

	/**
	 * Get the localization service provider that is currently in use.
	 */
	virtual ILocalizationServiceProvider& GetProvider() const = 0;

	/**
	 * Set the current localization service provider to the one specified here by name.
	 * This will assert if the provider does not exist.
	 * @param	InName	The name of the provider
	 */
	virtual void SetProvider( const FName& InName ) = 0;

	/**
	 * Get whether we should use global or per-project settings
	 * @return true if we should use global settings
	 */
	virtual bool GetUseGlobalSettings() const = 0;

	/**
	 * Set whether we should use global or per-project settings
	 * @param bIsUseGlobalSettings	Whether we should use global settings
	 */
	virtual void SetUseGlobalSettings(bool bIsUseGlobalSettings) = 0;

	/**
	 * Gets a reference to the localization service module instance.
	 *
	 * @return A reference to the localization service module.
	 */
	static inline ILocalizationServiceModule& Get()
	{
		static FName LocalizationServiceModule("LocalizationService");
		return FModuleManager::LoadModuleChecked<ILocalizationServiceModule>(LocalizationServiceModule);
	}
};
