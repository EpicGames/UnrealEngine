// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ILocalizationServiceOperation : public TSharedFromThis<ILocalizationServiceOperation, ESPMode::ThreadSafe>
{
public:
	/**
	 * Virtual destructor
	 */
	virtual ~ILocalizationServiceOperation() = default;

	/** Get the name of this operation, used as a unique identifier */
	virtual FName GetName() const = 0;

	/** Get the string to display when this operation is in progress */
	virtual FText GetInProgressString() const 
	{ 
		return FText(); 
	}

	/** Factory method for easier operation creation */
	template<typename Type>
	static TSharedRef<Type, ESPMode::ThreadSafe> Create()
	{
		return MakeShareable( new Type() );
	}
};

typedef TSharedRef<class ILocalizationServiceOperation, ESPMode::ThreadSafe> FLocalizationServiceOperationRef;
