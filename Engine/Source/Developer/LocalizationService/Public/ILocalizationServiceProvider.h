// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "ILocalizationServiceOperation.h"
#include "ILocalizationServiceState.h"
#include "Features/IModularFeature.h"
#include "LocalizationServiceOperations.h"

class IDetailCategoryBuilder;
class ULocalizationTarget;
class ULocalizationTargetSet;

//#include "LocalizationServiceHelpers.h"

class IDetailCategoryBuilder;
class ULocalizationTarget;
class ULocalizationTargetSet;

#define LOCALIZATION_SERVICES_WITH_SLATE	(!(PLATFORM_LINUX && IS_PROGRAM))

/**
 * Hint for how to execute the operation. Note that asynchronous operations require
 * Tick() to be called to manage completed operations.
 */
namespace ELocalizationServiceOperationConcurrency
{
	enum Type
	{
		/** Force the operation to be issued on the same thread, blocking until complete. */
		Synchronous,
		/** Run the command on another thread, returning immediately. */
		Asynchronous
	};
}

/**
* Hint to provider when updating state
*/
namespace ELocalizationServiceCacheUsage
{
	enum Type
	{
		/** Force a synchronous update of the state of the file. */
		ForceUpdate,
		/** Use the cached state if possible */
		Use,
	};
}

/**
 * Results of a command execution
 */
namespace ELocalizationServiceOperationCommandResult
{
	enum Type
	{
		/** Command failed to execute correctly or was not supported by the provider. */
		Failed,
		/** Command executed successfully */
		Succeeded,
		/** Command was canceled before completion */
		Cancelled,
	};
}

/** Delegate used by providers for when operations finish */
DECLARE_DELEGATE_TwoParams(FLocalizationServiceOperationComplete, const FLocalizationServiceOperationRef&, ELocalizationServiceOperationCommandResult::Type);

/** Delegate used by providers to create localization service operations */
typedef TSharedRef<class ILocalizationServiceOperation, ESPMode::ThreadSafe> FLocalizationServiceOperationSharedRef;
DECLARE_DELEGATE_RetVal(FLocalizationServiceOperationSharedRef, FGetLocalizationServiceOperation);

/**
 * Interface to talking with localization service providers.
 */
class ILocalizationServiceProvider : public IModularFeature
{
public:
	/**
	 * Virtual destructor
	 */
	virtual ~ILocalizationServiceProvider() = default;

	/** 
	 * Initialize localization service provider.
	 * @param	bForceConnection	If set, this flag forces the provider to attempt a connection to its server.
	 */
	virtual void Init(bool bForceConnection = true) = 0;

	/** 
	 * Shut down localization service provider.
	 */
	virtual void Close() = 0;

	/** Get the localization service provider name */
	virtual const FName& GetName() const = 0;

	/** Get the localization service provider display name */
	virtual const FText GetDisplayName() const = 0;

	/** Get the localization service status as plain, human-readable text */
	virtual FText GetStatusText() const = 0;

	/** Quick check if localization service is enabled */
	virtual bool IsEnabled() const = 0;

	/**
	 * Quick check if localization service is available for use (server-based providers can use this
	 * to return whether the server is available or not)
	 *
	 * @return	true if localization service is available, false if it is not
	 */
	virtual bool IsAvailable() const = 0;

	/**
	 * Login to the localization service server (if any).
	 * This is just a wrapper around Execute().
	 * @param	InPassword						The password (if required) to use when logging in.
	 * @param	InConcurrency					How to execute the operation, blocking or asynchronously on another thread.
	 * @param	InOperationCompleteDelegate		Delegate to call when the operation is completed. This is called back internal to this call when executed on the main thread, or from Tick() when queued for asynchronous execution.
	 * @return the result of the operation.
	 */
	virtual ELocalizationServiceOperationCommandResult::Type Login(const FString& InPassword = FString(), ELocalizationServiceOperationConcurrency::Type InConcurrency = ELocalizationServiceOperationConcurrency::Synchronous, const FLocalizationServiceOperationComplete& InOperationCompleteDelegate = FLocalizationServiceOperationComplete())
	{
		TSharedRef<FConnectToProvider, ESPMode::ThreadSafe> ConnectOperation = ILocalizationServiceOperation::Create<FConnectToProvider>();
		ConnectOperation->SetPassword(InPassword);
		return Execute(ConnectOperation, FLocalizationServiceTranslationIdentifier(), InConcurrency, InOperationCompleteDelegate);
	}

	/**
	* Get the state of each of the passed-in files. State may be cached for faster queries. Note states can be NULL!
	* @param	InTranslationIds	The translations to retrieve state for.
	* @param	OutState			The states of the files. This will be empty if the operation fails. Note states can be NULL!
	* @param	InStateCacheUsage	Whether to use the state cache or to force a (synchronous) state retrieval.
	* @return the result of the operation.
	*/
	virtual ELocalizationServiceOperationCommandResult::Type GetState(const TArray<FLocalizationServiceTranslationIdentifier>& InTranslationIds, TArray< TSharedRef<ILocalizationServiceState, ESPMode::ThreadSafe> >& OutState, ELocalizationServiceCacheUsage::Type InStateCacheUsage) = 0;

	/**
	* Helper overload for state retrieval, see GetState().
	*/
	virtual TSharedPtr<ILocalizationServiceState, ESPMode::ThreadSafe> GetState(const FLocalizationServiceTranslationIdentifier& InTranslationId, ELocalizationServiceCacheUsage::Type InStateCacheUsage)
	{
		TArray<FLocalizationServiceTranslationIdentifier> TranslationIds;
		TArray< TSharedRef<ILocalizationServiceState, ESPMode::ThreadSafe> > States;
		TranslationIds.Add(InTranslationId);
		if (GetState(TranslationIds, States, InStateCacheUsage) == ELocalizationServiceOperationCommandResult::Succeeded)
		{
			TSharedRef<ILocalizationServiceState, ESPMode::ThreadSafe> State = States[0];
			return State;
		}
		return NULL;
	}

	/**
	 * Attempt to execute an operation on the passed-in files (if any are required).
	 * @param	InTranslationIds				The translations in question
	 * @param	InOperation						The operation to perform.
	 * @param	InConcurrency					How to execute the operation, blocking or asynchronously on another thread.
	 * @param	InOperationCompleteDelegate		Delegate to call when the operation is completed. This is called back internal to this call when executed on the main thread, or from Tick() when queued for asynchronous execution.
	 * @return the result of the operation.
	 */
	virtual ELocalizationServiceOperationCommandResult::Type Execute(const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation, const TArray<FLocalizationServiceTranslationIdentifier>& InTranslationIds, ELocalizationServiceOperationConcurrency::Type InConcurrency = ELocalizationServiceOperationConcurrency::Synchronous, const FLocalizationServiceOperationComplete& InOperationCompleteDelegate = FLocalizationServiceOperationComplete()) = 0;

	/**
	 * Helper overload for operation execution, see Execute().
	 */
	virtual ELocalizationServiceOperationCommandResult::Type Execute(const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation, const ELocalizationServiceOperationConcurrency::Type InConcurrency = ELocalizationServiceOperationConcurrency::Synchronous, const FLocalizationServiceOperationComplete& InOperationCompleteDelegate = FLocalizationServiceOperationComplete())
	{
		return Execute(InOperation, TArray<FLocalizationServiceTranslationIdentifier>(), InConcurrency, InOperationCompleteDelegate);
	}

	/**
	 * Helper overload for operation execution, see Execute().
	 */
	virtual ELocalizationServiceOperationCommandResult::Type Execute(const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation, const FLocalizationServiceTranslationIdentifier& InTranslationId, const ELocalizationServiceOperationConcurrency::Type InConcurrency = ELocalizationServiceOperationConcurrency::Synchronous, const FLocalizationServiceOperationComplete& InOperationCompleteDelegate = FLocalizationServiceOperationComplete())
	{
		TArray<FLocalizationServiceTranslationIdentifier> TranslationIds;
		TranslationIds.Add(InTranslationId);
		return Execute(InOperation, TranslationIds, InConcurrency, InOperationCompleteDelegate);
	}

	/**
	 * Check to see if we can cancel an operation.
	 * @param	InOperation		The operation to check.
	 * @return true if the operation was cancelled.
	 */
	virtual bool CanCancelOperation( const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation ) const = 0;

	/**
	 * Attempt to cancel an operation in progress.
	 * @param	InOperation		The operation to attempt to cancel.
	 */
	virtual void CancelOperation( const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation ) = 0;

	/**
	 * Called every update.
	 */
	virtual void Tick() = 0;

#if LOCALIZATION_SERVICES_WITH_SLATE
	/**
	* Create a settings widget for display in the localization dashboard.
	* @returns a widget used to edit project-specific localization service settings.
	*/
	virtual void CustomizeSettingsDetails(IDetailCategoryBuilder& DetailCategoryBuilder) const = 0;

	/**
	* Create a settings widget for display in the localization target editor.
	* @returns a widget used to edit target-specific localization service settings.
	*/
	virtual void CustomizeTargetDetails(IDetailCategoryBuilder& DetailCategoryBuilder, TWeakObjectPtr<ULocalizationTarget> LocalizationTarget) const = 0;

	/**
	* Create a settings widget for display in the localization target editor.
	* @returns a widget used to edit target-specific localization service settings.
	*/
	virtual void CustomizeTargetToolbar(TSharedRef<FExtender>& MenuExtender, TWeakObjectPtr<ULocalizationTarget> LocalizationTarget) const = 0;

	/**
	* Create a settings widget for display in the localization target set editor.
	* @returns a widget used to edit target set-specific localization service settings.
	*/
	virtual void CustomizeTargetSetToolbar(TSharedRef<FExtender>& MenuExtender, TWeakObjectPtr<ULocalizationTargetSet> LocalizationTargetSet) const = 0;

#endif //LOCALIZATION_SERVICES_WITH_SLATE
};


