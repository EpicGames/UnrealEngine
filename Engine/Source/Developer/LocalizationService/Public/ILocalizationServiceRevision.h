// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** 
 * Abstraction of a localization revision from a localization service provider.
 */
class ILocalizationServiceRevision : public TSharedFromThis<ILocalizationServiceRevision, ESPMode::ThreadSafe>
{
public:
	/**
	 * Virtual destructor
	 */
	virtual ~ILocalizationServiceRevision() = default;

	/** Number of the revision */
	virtual int32 GetRevisionNumber() const = 0;

	/** String representation of the revision */
	virtual const FString& GetRevision() const = 0;

	/** User name of the submitter */
	virtual const FString& GetUserName() const = 0;

	/** Date of the revision */
	virtual const FDateTime& GetDate() const = 0;
};
