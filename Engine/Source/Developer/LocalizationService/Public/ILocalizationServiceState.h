// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

class ILocalizationServiceRevision;
class ILocalizationServiceState;

typedef TSharedRef<class ILocalizationServiceState, ESPMode::ThreadSafe> FLocalizationServiceStateRef;
typedef TSharedPtr<class ILocalizationServiceState, ESPMode::ThreadSafe> FLocalizationServiceStatePtr;

struct FLocalizationServiceTranslationIdentifier //: TSharedFromThis<FLocalizationServiceTranslationIdentifier, ESPMode::ThreadSafe>
{
	FLocalizationServiceTranslationIdentifier()
		: Culture(nullptr), Namespace(""), Source("")
	{}

	FLocalizationServiceTranslationIdentifier(FCulturePtr InCulturePtr, const FString& InNamespace, const FString& InSource)
		: LocalizationTargetGuid()
		, Culture(InCulturePtr)
		, Namespace(InNamespace)
		, Source(InSource)
	{

	}

	FGuid LocalizationTargetGuid;
	FCulturePtr Culture;
	FString Namespace;
	FString Source;
};

/**
 * An abstraction of the state of a file under localization service
 */
class ILocalizationServiceState : public TSharedFromThis<ILocalizationServiceState, ESPMode::ThreadSafe>
{
public:
	enum { INVALID_REVISION = -1 };

	/**
	 * Virtual destructor
	 */
	virtual ~ILocalizationServiceState() = default;

	/** 
	 * Get the size of the history. 
	 * If an FUpdateStatus operation has been called with the ShouldUpdateHistory() set, there 
	 * should be history present if the file has been committed to localization service.
	 * @returns the number of items in the history
	 */
	virtual int32 GetHistorySize() const = 0;

	/**
	* Get the text in question
	*/
	virtual const FString& GetSourceString() const = 0;

	/**
	* Get the text in question
	*/
	virtual const FString& GetTranslationString() const = 0;

	/**
	* Get the culture for which the localization of the text is for
	*/
	virtual const FCulturePtr GetCulture() const = 0;

	/**
	 * Get an item from the history
	 * @param	HistoryIndex	the index of the history item
	 * @returns a history item or NULL if none exist
	 */
	virtual TSharedPtr<class ILocalizationServiceRevision, ESPMode::ThreadSafe> GetHistoryItem( int32 HistoryIndex ) const = 0;

	/**
	* Get the name of the icon graphic we should use to display the state in a UI.
	* @returns the name of the icon to display
	*/
	virtual FName GetIconName() const = 0;

	/**
	* Get the name of the small icon graphic we should use to display the state in a UI.
	* @returns the name of the icon to display
	*/
	virtual FName GetSmallIconName() const = 0;

	/**
	 * Get a text representation of the state
	 * @returns	the text to display for this state
	 */
	virtual FText GetDisplayName() const = 0;

	/**
	 * Get a tooltip to describe this state
	 * @returns	the text to display for this states tooltip
	 */
	virtual FText GetDisplayTooltip() const = 0;

	///**
	// * Get a unique identifier for the translation this state represents
	// * @returns	the unique identifier
	// */
	virtual const FLocalizationServiceTranslationIdentifier& GetTranslationIdentifier() const = 0;

	/**
	 * Get the timestamp of the last update that was made to this state.
	 * @returns	the timestamp of the last update
	 */
	virtual const FDateTime& GetTimeStamp() const = 0;

	/** Get whether this translation is up-to-date with the version in localization service */
	virtual bool IsCurrent() const = 0;

	/** Get whether this text is known to the localization service */
	virtual bool IsKnownToLocalizationService() const = 0;

	/** Get whether localization service allows this translation to be edited */
	virtual bool CanEdit() const = 0;

	/** Get whether we know anything about this files localization service state */
	virtual bool IsUnknown() const = 0;

	/** Get whether this file is modified compared to the version we have from localization service */
	virtual bool IsModified() const = 0;
};

