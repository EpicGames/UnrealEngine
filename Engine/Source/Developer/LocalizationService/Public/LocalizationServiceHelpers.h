// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API LOCALIZATIONSERVICE_API

class FString;
class ILocalizationServiceProvider;

namespace LocalizationServiceHelpers
{
	/**
	 * Helper function to get the ini filename for storing localization service settings
	 * @return the filename
	 */
	LOCALIZATIONSERVICE_API extern const FString& GetSettingsIni();

	/**
	 * Helper function to get the ini filename for storing global localization service settings
	 * @return the filename
	 */
	LOCALIZATIONSERVICE_API extern const FString& GetGlobalSettingsIni();

	/**
	 * Helper function to commit a translation
	 */
	//LOCALIZATIONSERVICE_API extern bool CommitTranslation(const FString& Culture, const FString& Namespace, const FString& Source, const FString& Translation);
}

/**
* Helper class that ensures FLocalizationService is properly initialized and shutdown by calling Init/Close in
* its constructor/destructor respectively.
*/
class FScopedLocalizationService
{
public:
	/** Constructor; Initializes localization service Provider */
	UE_API FScopedLocalizationService();

	/** Destructor; Closes localization service Provider */
	UE_API ~FScopedLocalizationService();

	/** Get the provider we are using */
	UE_API ILocalizationServiceProvider& GetProvider();
};

#undef UE_API
