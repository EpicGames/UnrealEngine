// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "ILocalizationServiceOperation.h"

#define LOCTEXT_NAMESPACE "LocalizationService"

/**
 * Operation used to connect (or test a connection) to localization service
 */
class FConnectToProvider : public ILocalizationServiceOperation
{
public:
	// ILocalizationServiceOperation interface
	virtual FName GetName() const override 
	{ 
		return "Connect"; 
	}

	virtual FText GetInProgressString() const override
	{ 
		return LOCTEXT("LocalizationService_Connecting", "Connecting to localization service..."); 
	}

	const FString& GetPassword() const
	{
		return Password;
	}

	void SetPassword(const FString& InPassword)
	{
		Password = InPassword;
	}

	const FText& GetErrorText() const
	{
		return OutErrorText;
	}

	void SetErrorText(const FText& InErrorText)
	{
		OutErrorText = InErrorText;
	}

protected:
	/** Password we use for this operation */
	FString Password;

	/** Error text for easy diagnosis */
	FText OutErrorText;
};

/**
 * Operation used to download a localization target file from a localization service
 */
class FDownloadLocalizationTargetFile : public ILocalizationServiceOperation
{
public:

	// ILocalizationServiceOperation interface
	virtual FName GetName() const override
	{
		return "DownloadLocalizationTargetFile";
	}

	virtual FText GetInProgressString() const override
	{ 
		return LOCTEXT("LocalizationService_FLocalizationServiceDownloadLocalizationTargetFile", "Getting Localization Target File from localization Service..."); 
	}


	void SetInTargetGuid(const FGuid& NewInTargetGuid)
	{
		InTargetGuid = NewInTargetGuid;
	}

	const FGuid& GetInTargetGuid() const
	{
		return InTargetGuid;
	}

	void SetInRelativeOutputFilePathAndName(const FString& NewInRelativeOutputFilePathAndName)
	{
		InRelativeOutputFilePathAndName = NewInRelativeOutputFilePathAndName;
	}

	const FString& GetInRelativeOutputFilePathAndName() const
	{
		return InRelativeOutputFilePathAndName;
	}

	FString GetInLocale()
	{
		return InLocale;
	}

	void SetInLocale(FString NewLocale)
	{
		InLocale = NewLocale;
	}

	FText GetOutErrorText()
	{
		return OutErrorText;
	}

	void SetOutErrorText(const FText& NewOutErrorText)
	{
		OutErrorText = NewOutErrorText;
	}


protected:

	/** The GUID of the Localization Target */
	FGuid InTargetGuid;

	/** The locale (culture code, for example "fr" or "en-us" or "ja-jp") to download the translations for */
	FString InLocale;

	/** The path and name to the file downloaded, relative to project directory */
	FString InRelativeOutputFilePathAndName;

	/** Place to easily store and access error message */
	FText OutErrorText;
};


/**
* Operation used to upload a localization file to a localization service
*/
class FUploadLocalizationTargetFile : public ILocalizationServiceOperation
{
public:

	FUploadLocalizationTargetFile() : bPreserveAllText(true) {}

	// ILocalizationServiceOperation interface
	virtual FName GetName() const override
	{
		return "UploadLocalizationTargetFile";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("LocalizationService_FLocalizationServiceUploadLocalizationTargetFile", "Uploading Localization Target File to localization Service...");
	}


	void SetInTargetGuid(const FGuid& NewInTargetGuid)
	{
		InTargetGuid = NewInTargetGuid;
	}

	const FGuid& GetInTargetGuid() const
	{
		return InTargetGuid;
	}

	void SetInRelativeInputFilePathAndName(const FString& NewInRelativeInputFilePathAndName)
	{
		InRelativeInputFilePathAndName = NewInRelativeInputFilePathAndName;
	}

	const FString& GetInRelativeInputFilePathAndName() const
	{
		return InRelativeInputFilePathAndName;
	}

	FString GetInLocale()
	{
		return InLocale;
	}

	void SetInLocale(FString NewLocale)
	{
		InLocale = NewLocale;
	}

	FText GetOutErrorText()
	{
		return OutErrorText;
	}

	void SetOutErrorText(const FText& NewOutErrorText)
	{
		OutErrorText = NewOutErrorText;
	}

	bool GetPreserveAllText()
	{
		return bPreserveAllText;
	}

	void SetPreserveAllText(bool NewPreserveAllText)
	{
		bPreserveAllText = NewPreserveAllText;
	}


protected:

	/** The GUID of the Localization Target */
	FGuid InTargetGuid;

	/** The locale (culture code, for example "fr" or "en-us" or "ja-jp") to download the translations for */
	FString InLocale;

	/** The path and name to the file to upload, relative to project directory */
	FString InRelativeInputFilePathAndName;

	/** Place to easily store and access error message */
	FText OutErrorText;

	/** Ask the translations service to keep all text, even if not present in the current upload file.*/
	bool bPreserveAllText;
};


#undef LOCTEXT_NAMESPACE
