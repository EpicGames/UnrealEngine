// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWindow.h"

class FScopedLocalizationServiceProgress
{
public:
	/** 
	 * Constructor.
	 * Progress dialog is shown immediately (if possible) and is closed when this
	 * object goes out of scope.
	 * @param	InText			The text to display in the progress dialog. If this is empty the dialog will not be shown.
	 * @param	InOnCancelled	The delegate to call when the cancel button is clicked.
	 */
	LOCALIZATIONSERVICE_API FScopedLocalizationServiceProgress(const FText& InText, const FSimpleDelegate& InOnCancelled = FSimpleDelegate());

	/** Destructor - closes the dialog */
	LOCALIZATIONSERVICE_API ~FScopedLocalizationServiceProgress();

	/** Makes sure the dialog is ticked */
	LOCALIZATIONSERVICE_API void Tick();

private:

	/** The window containing this progress window */
	TWeakPtr<class SWindow> WindowPtr;
};
